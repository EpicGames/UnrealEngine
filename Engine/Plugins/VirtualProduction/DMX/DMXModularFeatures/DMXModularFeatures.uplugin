{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "DMX Modular Features",
	"Description": "Modular Features for DMX",
	"Category": "Misc",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Hidden": true,
	"Modules": [
		{
			"Name": "DMXFixtureActorInterface",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}
