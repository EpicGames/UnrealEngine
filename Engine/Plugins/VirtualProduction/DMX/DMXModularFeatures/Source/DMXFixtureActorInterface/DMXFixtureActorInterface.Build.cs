// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DMXFixtureActorInterface : ModuleRules
{
	public DMXFixtureActorInterface(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
			});
	}
}
