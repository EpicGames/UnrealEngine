// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "DMXMVRFixtureActorInterface.generated.h"

class UDMXEntityFixturePatch;


/** 
 * When implemented in an actor, MVR will find the Actor and consider it when auto-selecting Fixtures.
 * 
 * Note the Actor that implements this is required to have exactly one DMXComponent subobject. 
 * It is the DMX Component that will write its MVR UUID to the Actor's MetaData and make the Actor identified as an MVR Actor.
 */
UINTERFACE(BlueprintType)
class DMXFIXTUREACTORINTERFACE_API UDMXMVRFixtureActorInterface
	: public UInterface
{
	GENERATED_BODY()

};

class DMXFIXTUREACTORINTERFACE_API IDMXMVRFixtureActorInterface
{
	GENERATED_BODY()

public:
	/** Should return the DMX Attributes the Actor supports */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "DMX", Meta = (DisplayName = "On MVR Get Supported DMX Attributes"))
	void OnMVRGetSupportedDMXAttributes(TArray<FName>& OutAttributeNames, TArray<FName>& OutMatrixAttributeNames) const;
};
